// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class Phya : ModuleRules
	{
        public Phya(ReadOnlyTargetRules Target) : base(Target)
		{
            PrivateIncludePaths.Add("Phya/Private/PhyaLib/include");

			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
                    "Engine"
				}
				);
		}
	}
}
